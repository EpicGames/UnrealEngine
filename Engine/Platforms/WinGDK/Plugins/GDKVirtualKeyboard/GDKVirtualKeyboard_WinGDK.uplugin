{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": [
		"WinGDK"
	],
	"Modules": [
		{
			"Name": "GDKVirtualKeyboard",
			"Type": "Runtime",
			"PlatformAllowList": [
				"WinGDK"
			]
		}
	]
}