{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "GDK Virtual Keyboard",
	"Description": "Virtual Keyboard support for GDK. Used for Slate text input etc",
	"CreatedBy": "Epic Games, Inc.",
	"Category": "Platform",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"CanContainContent": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "GDKVirtualKeyboard",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "Win64" ]
		}
	]
}