// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using EpicGames.Core;

public class GDKVirtualKeyboard : ModuleRules
{
	public GDKVirtualKeyboard(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.AddAll( 
			"SlateCore",
			"Slate",
			"GDKRuntime",
			"GRDK"
		);

		PublicDependencyModuleNames.AddAll(
			"Core"
		);
	}
}
