// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKPlatformTextField.h"

#if WITH_GRDK
#include "Widgets/Input/IVirtualKeyboardEntry.h"
#include "Containers/Ticker.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformTime.h"
#include "SlateGlobals.h"
#include "SlateSettings.h"
#include "UObject/Class.h"
#include "Misc/ScopeLock.h"
#include "GDKRuntimeModule.h"
#include "Containers/Ticker.h"
#include "Async/Async.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XGameUI.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
static TAutoConsoleVariable<int32> CVarAutoCancelKeyboard(
	TEXT("GDKPlatformTextField.AutoCancel"),
	0,
	TEXT("Automatically cancel platform text field shortly after opening it.\n")
	TEXT(" 0: Do not cancel(default)\n")
	TEXT("Otherwise: Approx number of seconds before cancelling\n"),
	ECVF_Default
);
#endif

static TAutoConsoleVariable<float> CVarGDKVirtualKeyboardDebounceDelaySeconds(
	TEXT("GDKPlatformTextField.DebounceDelaySeconds"),
	0.25f,
	TEXT("Delay to prevent the platform text field from re-opening immediately after it has been closed")
);

struct FAsyncKeyboardEntryUserData
{
	FAsyncKeyboardEntryUserData( FGDKPlatformTextField* InTextField, TWeakPtr<IVirtualKeyboardEntry> InWeakTextEntryWidget )
		: TextField(InTextField)
		, WeakTextEntryWidget(InWeakTextEntryWidget)
	{
		IGDKRuntimeModule::Get().Internal_HandlePlatformTextFieldVisible(true);
	}

	~FAsyncKeyboardEntryUserData()
	{
		IGDKRuntimeModule::Get().Internal_HandlePlatformTextFieldVisible(false);
	}

	FGDKPlatformTextField* TextField;
	TWeakPtr<IVirtualKeyboardEntry> WeakTextEntryWidget;
};

FGDKPlatformTextField::FGDKPlatformTextField()
{
	DebounceTime = 0.0f;
}

FGDKPlatformTextField::~FGDKPlatformTextField()
{
	KillExisitingDialog();
}

void FGDKPlatformTextField::KillExisitingDialog()
{
	check(IsInGameThread());

	FScopeLock Lock(&AsyncBlockCrit);

	// Lock is released here so we don't deadlock if the callback tries to acquire it while we're trying to cancel
	if (KeyboardAsyncBlock.IsValid())
	{
		XAsyncCancel(KeyboardAsyncBlock->GetInnerBlockForGDKAPI());
		(void)KeyboardAsyncBlock.Release();
	}
}

void FGDKPlatformTextField::ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget)
{
	bool bCanShow = (FPlatformTime::Seconds() >= DebounceTime);

	KillExisitingDialog();

	if (bShow && bCanShow)
	{
		XGameUiTextEntryInputScope InputScope = XGameUiTextEntryInputScope::Default;
		switch (TextEntryWidget->GetVirtualKeyboardType())
		{
		case EKeyboardType::Keyboard_Number:
			InputScope = XGameUiTextEntryInputScope::Number;
			break;
		case EKeyboardType::Keyboard_Web:
			InputScope = XGameUiTextEntryInputScope::Url;
			break;
		case EKeyboardType::Keyboard_Email:
			InputScope = XGameUiTextEntryInputScope::EmailSmtpAddress;
			break;
		case EKeyboardType::Keyboard_Password:
			InputScope = XGameUiTextEntryInputScope::Password;
			break;
		case EKeyboardType::Keyboard_AlphaNumeric:
			InputScope = XGameUiTextEntryInputScope::Alphanumeric;
			break;
		case EKeyboardType::Keyboard_Default:
		default:
			InputScope = XGameUiTextEntryInputScope::Default;
			break;
		}

		check(KeyboardAsyncBlock.IsValid() == false);
		
		FAsyncKeyboardEntryUserData* UserData = new FAsyncKeyboardEntryUserData( this, TextEntryWidget );
		KeyboardAsyncBlock = MakeUnique<FGDKAsyncBlock>(UserData, [](FGDKAsyncBlock* AsyncBlock)
		{
			// Take ownership of the userdata 
			TUniquePtr<FAsyncKeyboardEntryUserData> UserData((FAsyncKeyboardEntryUserData*)AsyncBlock->GetUserData());
			FGDKPlatformTextField* TextField = UserData->TextField;
			TWeakPtr<IVirtualKeyboardEntry> WeakTextEntryWidget = UserData->WeakTextEntryWidget;

			HRESULT Status = XAsyncGetStatus( AsyncBlock->GetInnerBlockForGDKAPI(), false );

			// Only call into background thread functions, don't access any resources created on the main thread.
			switch (Status)
			{
				case S_OK:
				{
					uint32 ResultSize = 0;
					XGameUiShowTextEntryResultSize( AsyncBlock->GetInnerBlockForGDKAPI(), &ResultSize );						
					char* ResultTextBuffer = new char[ResultSize+1];
					XGameUiShowTextEntryResult( AsyncBlock->GetInnerBlockForGDKAPI(), ResultSize, ResultTextBuffer, nullptr );

					FString StringResult(UTF8_TO_TCHAR(ResultTextBuffer));
					delete[] ResultTextBuffer;

					TextField->VirtualKeyboardCallbackBackgroundThread(WeakTextEntryWidget, StringResult, ETextEntryType::TextEntryAccepted);
					break;
				}
				default:
				{
					UE_LOGF(LogSlate, Verbose, "Virtual Keyboard returned without text due to an error state %x.", Status);
					// Error falls through to the canceled case
				}
				case E_ABORT:
				{
					// Canceled returns original text and sends the TextEntryCanceld status
					TextField->VirtualKeyboardCallbackBackgroundThread(WeakTextEntryWidget, TEXT(""), ETextEntryType::TextEntryCanceled);
					break;
				}
			}

			FScopeLock Lock(&TextField->AsyncBlockCrit);
			// We may have raced with KillExisitingDialog so we acquire the lock to make sure they aren't trying to cancel us 
			if (TextField->KeyboardAsyncBlock.Get() == AsyncBlock)
			{
				(void)TextField->KeyboardAsyncBlock.Release();
			}

			delete AsyncBlock;
		});

		HRESULT Result = XGameUiShowTextEntryAsync( KeyboardAsyncBlock->GetInnerBlockForGDKAPI(), 
													TCHAR_TO_UTF8( *TextEntryWidget->GetHintText().ToString() ),
													"",
													TCHAR_TO_UTF8( *TextEntryWidget->GetText().ToString() ),
													InputScope,
													0 ); //0 = no limit
		if( Result != S_OK )
		{
			// No synchronization here because we cancelled any outstanding task with KillExisitingDialog and 
			// released KeyboardAsyncBlock so we own the pointer in there and any outstanding cancellation refers to a 
			// different block
			delete UserData;
			UserData = nullptr;
			KeyboardAsyncBlock.Reset();

			if (Result == E_ACCESSDENIED)
			{
				// The application did not have focus. A system UI is up or we are constrained.
				UE_LOGF(LogSlate, Verbose, "Virtual Keyboard was invoked when the application did not have focus.");
				return;
			}

			UE_LOGF(LogSlate, Warning, "Virtual Keyboard failed with exception %x", Result);
			return;
		}
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
		else 
		{
			if (int32 Secs = CVarAutoCancelKeyboard.GetValueOnGameThread())
			{
				Async(EAsyncExecution::ThreadPool, [this, Secs]() {
					FPlatformProcess::Sleep((float)Secs);
					ExecuteOnGameThread(UE_SOURCE_LOCATION, [this]() { KillExisitingDialog();  });
				});
			}
		}
#endif
	}
	else if (bShow && !bCanShow)
	{
		UE_LOGF(LogSlate, Warning, "Ignoring Virtual Keyboard show request because it was closed too recently");
	}

}

void FGDKPlatformTextField::VirtualKeyboardCallbackBackgroundThread(TWeakPtr<IVirtualKeyboardEntry> WeakTextEntryWidget, FString StringResult, ETextEntryType EntryType)
{
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [this, WeakTextEntryWidget, StringResult, EntryType]() {
		VirtualKeyboardCallbackGameThread(WeakTextEntryWidget, StringResult, EntryType);
	});
}

void FGDKPlatformTextField::VirtualKeyboardCallbackGameThread(TWeakPtr<IVirtualKeyboardEntry> WeakTextEntryWidget, FString StringResult, ETextEntryType EntryType)
{
	check(IsInGameThread());
	
	// Don't touch KeyboardAsyncBlock here - it should have already been deleted by the callback or KillExisitingDialog
	// If it's not null here, that means we re-opened the dialog and this callback doesn't own that block

	if (TSharedPtr<IVirtualKeyboardEntry> StrongTextEntryWidget = WeakTextEntryWidget.Pin())
	{
		switch (EntryType)
		{
		case ETextEntryType::TextEntryAccepted:
			StrongTextEntryWidget->SetTextFromVirtualKeyboard(FText::FromString(StringResult), ETextEntryType::TextEntryAccepted);
			break;
		case ETextEntryType::TextEntryCanceled:
			StrongTextEntryWidget->SetTextFromVirtualKeyboard(StrongTextEntryWidget->GetText(), ETextEntryType::TextEntryCanceled);
			break;
		default:
			check(false);
		}
	}

	// prevent the virtual keyboard from opening again immediately
	DebounceTime = FPlatformTime::Seconds() + CVarGDKVirtualKeyboardDebounceDelaySeconds.GetValueOnGameThread();
}

#endif //WITH_GRDK

