// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"
#include "Framework/Application/IPlatformTextField.h"

#if WITH_GRDK
#include "GDKPlatformTextField.h"

class FGDKVirtualKeyboardModule : public IModuleInterface, public IPlatformTextFieldFactory
{
public:

	// IModuleInterface
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(IPlatformTextFieldFactory::FeatureName, this);
	}

	virtual void ShutdownModule() override
	{
		IModularFeatures::Get().UnregisterModularFeature(IPlatformTextFieldFactory::FeatureName, this);
	}

	// IPlatformTextFieldFactory
	virtual TUniquePtr<IPlatformTextField> CreateInstance() override
	{
		return MakeUnique<FGDKPlatformTextField>();
	}
};

IMPLEMENT_MODULE(FGDKVirtualKeyboardModule, GDKVirtualKeyboard);

#else

IMPLEMENT_MODULE(FDefaultModuleImpl, GDKVirtualKeyboard);

#endif //WITH_GRDK
