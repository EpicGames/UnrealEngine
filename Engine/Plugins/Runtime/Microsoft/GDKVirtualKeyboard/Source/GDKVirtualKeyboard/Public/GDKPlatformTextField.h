// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Application/IPlatformTextField.h"

#if WITH_GRDK
#include "GDKTaskQueueHelpers.h"

class FGDKPlatformTextField : public IPlatformTextField
{
public:
	FGDKPlatformTextField();
	~FGDKPlatformTextField();

	virtual void ShowVirtualKeyboard(bool bShow, int32 UserIndex, TSharedPtr<IVirtualKeyboardEntry> TextEntryWidget) override;

private:
	void VirtualKeyboardCallbackBackgroundThread(TWeakPtr<IVirtualKeyboardEntry> WeakTextEntryWidget, FString StringResult, ETextEntryType EntryType);
	void VirtualKeyboardCallbackGameThread(TWeakPtr<IVirtualKeyboardEntry> WeakTextEntryWidget, FString StringResult, ETextEntryType EntryType);

	void KillExisitingDialog();

	FCriticalSection AsyncBlockCrit;
	TUniquePtr<FGDKAsyncBlock> KeyboardAsyncBlock;
	double DebounceTime;
};

#endif //WITH_GRDK
